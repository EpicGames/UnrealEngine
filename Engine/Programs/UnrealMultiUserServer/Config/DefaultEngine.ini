[URL]
GameNameShort=MUS

[Plugins]
+ProgramEnabledPlugins="UdpMessaging"
+ProgramEnabledPlugins="QuicMessaging"
+ProgramEnabledPlugins="ConcertSyncServer"

[/Script/MacTargetPlatform.XcodeProjectSettings]
TemplateMacPlist=(FilePath="/Engine/Source/Programs/UnrealMultiUserServer/Resources/Mac/Info.plist")
